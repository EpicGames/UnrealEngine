[CoreRedirects]

; Rename HierarchyTableBuiltin to HierarchyTableAnimation
+StructRedirects=(OldName="/Script/HierarchyTableBuiltinRuntime.HierarchyTable_ElementType_Mask", NewName="/Script/HierarchyTableAnimationRuntime.HierarchyTable_ElementType_Mask")
+StructRedirects=(OldName="/Script/HierarchyTableBuiltinRuntime.HierarchyTable_ElementType_Time", NewName="/Script/HierarchyTableAnimationRuntime.HierarchyTable_ElementType_Time")
+StructRedirects=(OldName="/Script/HierarchyTableBuiltinRuntime.HierarchyTable_TableType_Skeleton", NewName="/Script/HierarchyTableAnimationRuntime.HierarchyTable_TableType_Skeleton")
+StructRedirects=(OldName="/Script/HierarchyTableBuiltinRuntime.HierarchyTable_TablePayloadType_Skeleton", NewName="/Script/HierarchyTableAnimationRuntime.HierarchyTable_TablePayloadType_Skeleton")