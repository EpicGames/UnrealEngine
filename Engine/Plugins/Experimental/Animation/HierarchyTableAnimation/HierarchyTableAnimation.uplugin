{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Hierarchy Table Animation",
	"Description": "Animation-specific type definitions for Hierarchy Tables",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "HierarchyTableAnimationRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HierarchyTableAnimationEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HierarchyTableAnimationUncookedOnly",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "HierarchyTable",
			"Enabled": true
		}
	]
}
