// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

public class HierarchyTableAnimationEditor : ModuleRules
{
	public HierarchyTableAnimationEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "HTAnimEd";

		PrivateDependencyModuleNames.AddAll(
			"AssetDefinition",
			"Core",
			"CoreUObject",
			"Engine",
			"EditorWidgets",
			"InputCore",
			"Slate",
			"SlateCore",
			"UnrealEd",
			"HierarchyTableRuntime",
			"HierarchyTableEditor",
			"HierarchyTableAnimationRuntime",
			"ToolMenus",
			"PropertyEditor",
			"Persona"
		);
	}
}
