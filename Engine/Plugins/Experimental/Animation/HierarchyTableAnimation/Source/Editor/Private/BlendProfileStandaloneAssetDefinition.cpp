// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendProfileStandaloneAssetDefinition.h"
#include "BlendProfileStandalone.h"
#include "BlendProfileStandaloneEditorToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendProfileStandaloneAssetDefinition)

#define LOCTEXT_NAMESPACE "BlendProfileStandalone"

FText UAssetDefinition_BlendProfileStandalone::GetAssetDisplayName() const
{
	return LOCTEXT("BlendProfileStandalone", "Blend Profile");
}

FLinearColor UAssetDefinition_BlendProfileStandalone::GetAssetColor() const
{
	return FLinearColor(FColor::Yellow);
}

TSoftClassPtr<UObject> UAssetDefinition_BlendProfileStandalone::GetAssetClass() const
{
	return UBlendProfileStandalone::StaticClass();
}

EAssetCommandResult UAssetDefinition_BlendProfileStandalone::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	if (OpenArgs.OpenMethod == EAssetOpenMethod::Edit)
	{
		TArray<UObject*> Assets = OpenArgs.LoadObjects<UObject>();
		MakeShared<FBlendProfileStandaloneEditorToolkit>()->InitEditor(Assets);
	}

	return EAssetCommandResult::Handled;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_BlendProfileStandalone::GetAssetCategories() const
{
	static const auto Categories = { FAssetCategoryPath(EAssetCategoryPaths::Animation) };
	return Categories;
}

#undef LOCTEXT_NAMESPACE
