// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendProfileStandaloneEditorToolkit.h"
#include "HierarchyTableEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "BlendProfileStandaloneEditorToolkit"

void FBlendProfileStandaloneEditorToolkit::InitEditor(const TArray<UObject*>& InObjects)
{
	BlendProfile = CastChecked<UBlendProfileStandalone>(InObjects[0]);
	
	BlendProfile->Table->SetFlags(RF_Transactional);
	BlendProfile->UpdateHierarchy();

	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("BlendProfileStandaloneEditorToolkit")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.7f)
				->AddTab("BlendProfileStandaloneEditorTableTab", ETabState::OpenedTab)
			)
			->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(0.3f)
				->AddTab("BlendProfileStandaloneEditorDetailsTab", ETabState::OpenedTab)
			)
		);

	FAssetEditorToolkit::InitAssetEditor(EToolkitMode::Standalone, {}, "BlendProfileStandaloneEditor", Layout, true, true, InObjects);

	ExtendToolbar();

	TObjectPtr<USkeleton> Skeleton = BlendProfile->GetSkeleton();
	if (Skeleton)
	{
		Skeleton->RegisterOnSkeletonHierarchyChanged(USkeleton::FOnSkeletonHierarchyChanged::CreateRaw(this, &FBlendProfileStandaloneEditorToolkit::OnSkeletonHierarchyChanged));
	}
}

void FBlendProfileStandaloneEditorToolkit::OnClose()
{
	TObjectPtr<USkeleton> Skeleton = BlendProfile->GetSkeleton();
	if (Skeleton)
	{
		Skeleton->UnregisterOnSkeletonHierarchyChanged(this);
	}
}

void FBlendProfileStandaloneEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("BlendProfileStandaloneEditor", "Blend Profile Editor"));

	// Table view
	{
		FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");

		TSharedRef<IHierarchyTable> TableWidget = HierarchyTableModule.CreateHierarchyTableWidget(BlendProfile->Table);
		HierarchyTableWidgetInterface = TableWidget;

		InTabManager->RegisterTabSpawner("BlendProfileStandaloneEditorTableTab", FOnSpawnTab::CreateLambda([this, TableWidget](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
					[
						TableWidget
					];
			}))
			.SetDisplayName(LOCTEXT("BlendProfile", "Blend Profile"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}

	// Details panel
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsView->SetObjects(TArray<UObject*>{ BlendProfile });

		InTabManager->RegisterTabSpawner("BlendProfileStandaloneEditorDetailsTab", FOnSpawnTab::CreateLambda([=](const FSpawnTabArgs&)
			{
				return SNew(SDockTab)
					[
						DetailsView
					];
			}))
			.SetDisplayName(INVTEXT("Details"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}
}

void FBlendProfileStandaloneEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
	InTabManager->UnregisterTabSpawner("BlendProfileStandaloneEditorTableTab");
	InTabManager->UnregisterTabSpawner("BlendProfileStandaloneEditorDetailsTab");
}

void FBlendProfileStandaloneEditorToolkit::OnSkeletonHierarchyChanged()
{
	BlendProfile->UpdateHierarchy();
	BlendProfile->UpdateCachedData();
}

void FBlendProfileStandaloneEditorToolkit::ExtendToolbar()
{
	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");
	const TObjectPtr<UHierarchyTable_TableTypeHandler> Handler = HierarchyTableModule.CreateTableHandler(BlendProfile->Table);
	if (!ensure(Handler))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(this);

	FName ParentName;
	static const FName MenuName = GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	Handler->ExtendToolbar(ToolMenu, *HierarchyTableWidgetInterface);
}

#undef LOCTEXT_NAMESPACE