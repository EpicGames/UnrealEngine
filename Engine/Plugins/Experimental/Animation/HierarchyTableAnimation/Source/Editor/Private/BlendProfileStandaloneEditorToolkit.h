// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlendProfileStandalone.h"
#include "Toolkits/AssetEditorToolkit.h"

class SHierarchyTable;
class IHierarchyTable;

class FBlendProfileStandaloneEditorToolkit : public FAssetEditorToolkit
{
public:
	void InitEditor(const TArray<UObject*>& InObjects);

	// Begin FAssetEditorToolkit
	void OnClose() override;

	void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	FName GetToolkitFName() const override { return "BlendProfileEditor"; }
	FText GetBaseToolkitName() const override { return INVTEXT("Blend Profile Editor"); }
	FString GetWorldCentricTabPrefix() const override { return "Blend Profile "; }
	FLinearColor GetWorldCentricTabColorScale() const override { return {}; }
	// End FAssetEditorToolkit

private:
	void OnSkeletonHierarchyChanged();

	void ExtendToolbar();

	TObjectPtr<UBlendProfileStandalone> BlendProfile;

	TSharedPtr<IHierarchyTable> HierarchyTableWidgetInterface;
};
