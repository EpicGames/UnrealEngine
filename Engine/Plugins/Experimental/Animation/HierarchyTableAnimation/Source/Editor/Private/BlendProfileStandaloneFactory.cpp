// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendProfileStandaloneFactory.h"
#include "BlendProfileStandalone.h"
#include "Editor.h"
#include "HierarchyTableEditorModule.h"
#include "MaskProfile/HierarchyTableTypeMask.h"
#include "Modules/ModuleManager.h"
#include "SEnumCombo.h"
#include "SkeletonHierarchyTableType.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendProfileStandaloneFactory)

#define LOCTEXT_NAMESPACE "BlendProfileStandaloneFactory"

UBlendProfileStandaloneFactory::UBlendProfileStandaloneFactory()
	: BlendProfileType(EBlendProfileStandaloneType::WeightFactor)
{
	SupportedClass = UBlendProfileStandalone::StaticClass();
	bCreateNew = true;
}

UObject* UBlendProfileStandaloneFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	TObjectPtr<UBlendProfileStandalone> BlendProfile = NewObject<UBlendProfileStandalone>(InParent, Class, Name, Flags, Context);
	BlendProfile->Table = NewObject<UHierarchyTable>(BlendProfile);
	BlendProfile->Type = BlendProfileType;

	// TODO: Streamline hierarchy table creation API

	BlendProfile->Table->Initialize(TableMetadata, FHierarchyTable_ElementType_Mask::StaticStruct());

	check(TableHandler);
	TableHandler->SetHierarchyTable(BlendProfile->Table);
	TableHandler->ConstructHierarchy();

	return BlendProfile;
}

bool UBlendProfileStandaloneFactory::ConfigureProperties()
{
	if (!ConfigureBlendProfileType())
	{
		return false;
	}

	return ConfigureBlendProfileHierarchy();
}

bool UBlendProfileStandaloneFactory::ConfigureBlendProfileType()
{
	bool bConfirmClicked = false;

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("Title", "Choose Blend Profile Type"))
		.ClientSize(FVector2D(400, 400))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SEnumComboBox, StaticEnum<EBlendProfileStandaloneType>())
						.CurrentValue_Lambda([this]()
							{
								return static_cast<int32>(BlendProfileType);
							})
						.OnEnumSelectionChanged_Lambda([this](int32 InEnumValue, ESelectInfo::Type SelectInfo)
							{
								BlendProfileType = static_cast<EBlendProfileStandaloneType>(InEnumValue);
							})
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SButton)
						.OnClicked_Lambda([&]()
							{
								bConfirmClicked = true;
								Window->RequestDestroyWindow();
								return FReply::Handled();
							})
						[
							SNew(STextBlock)
								.Text(LOCTEXT("Confirm", "Confirm"))
						]
				]
		];


	GEditor->EditorAddModalWindow(Window);

	return bConfirmClicked;
}

bool UBlendProfileStandaloneFactory::ConfigureBlendProfileHierarchy()
{
	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");

	TableHandler = HierarchyTableModule.CreateTableHandler(FHierarchyTable_TableType_Skeleton::StaticStruct());
	check(TableHandler);

	TableMetadata = FInstancedStruct(FHierarchyTable_TableType_Skeleton::StaticStruct());

	// Displays window for setting skeleton
	return TableHandler->FactoryConfigureProperties(TableMetadata);
}

#undef LOCTEXT_NAMESPACE
