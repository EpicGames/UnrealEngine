// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "StructUtils/InstancedStruct.h"

#include "BlendProfileStandaloneFactory.generated.h"

enum class EBlendProfileStandaloneType;
class UHierarchyTable_TableTypeHandler;

UCLASS()
class UBlendProfileStandaloneFactory : public UFactory
{
	GENERATED_BODY()

public:
	UBlendProfileStandaloneFactory();

	// UFactory interface
	UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ConfigureProperties() override;

private:
	bool ConfigureBlendProfileType();

	bool ConfigureBlendProfileHierarchy();

	EBlendProfileStandaloneType BlendProfileType;

	FInstancedStruct TableMetadata;

	UPROPERTY()
	TObjectPtr<UHierarchyTable_TableTypeHandler> TableHandler;
};
