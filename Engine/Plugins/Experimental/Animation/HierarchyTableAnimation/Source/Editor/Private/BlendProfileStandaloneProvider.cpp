// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendProfileStandaloneProvider.h"

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "ContentBrowserModule.h"
#include "PropertyCustomizationHelpers.h"
#include "HierarchyTableBlendProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendProfileStandaloneProvider)

void UBlendProfileStandaloneProvider::Initialize(TObjectPtr<UBlendProfileStandalone> InBlendProfile)
{
	BlendProfile = InBlendProfile;
}

void UBlendProfileStandaloneProvider::ConstructBlendProfile(const TObjectPtr<UBlendProfile> OutBlendProfile) const
{
	if (BlendProfile && BlendProfile->Table)
	{
		// Switch from EBlendProfileStandaloneType to EBlendProfileMode.
		// These enums have the same values but EBlendProfileMode marks BlendMask as hidden 
		// so it does not appear in UIs which is a problem for us so we define our own.

		EBlendProfileMode Mode = EBlendProfileMode::WeightFactor;
		if (BlendProfile->Type == EBlendProfileStandaloneType::TimeFactor)
		{
			Mode = EBlendProfileMode::TimeFactor;
		}
		else if (BlendProfile->Type == EBlendProfileStandaloneType::BlendMask)
		{
			Mode = EBlendProfileMode::BlendMask;
		}

		FHierarchyTableBlendProfile HierarchyTableBlendProfile(BlendProfile->Table, Mode);
		HierarchyTableBlendProfile.ConstructBlendProfile(OutBlendProfile);
	}
}

FName FBlendProfileStandalonePickerExtender::StaticGetId()
{
	return "Standalone";
}

FName FBlendProfileStandalonePickerExtender::GetId() const
{
	return StaticGetId();
}

FText  FBlendProfileStandalonePickerExtender::GetDisplayName() const
{
	return NSLOCTEXT("BlendProfileStandalone", "Asset", "Asset");
}

TSharedRef<SWidget> FBlendProfileStandalonePickerExtender::ConstructPickerWidget(const FPickerWidgetArgs& InWidgetArgs) const
{
	return SNew(FBlendProfileStandalonePickerExtender::SPicker, InWidgetArgs);
}

bool FBlendProfileStandalonePickerExtender::OwnsBlendProfileProvider(const TObjectPtr<const UObject> InObject) const
{
	return InObject.IsA<UBlendProfileStandaloneProvider>();
}

void FBlendProfileStandalonePickerExtender::SPicker::Construct(const FArguments& InArgs, IBlendProfilePickerExtender::FPickerWidgetArgs InPickerArgs)
{
	TObjectPtr<UBlendProfileStandaloneProvider> InitialProvider = Cast<UBlendProfileStandaloneProvider>(InPickerArgs.InitialSelection);
	SelectedAsset = InitialProvider ? FAssetData(InitialProvider->BlendProfile) : FAssetData();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	ChildSlot
		[
			SNew(SObjectPropertyEntryBox)
				.ObjectPath_Lambda([this]()
					{
						return SelectedAsset.GetObjectPathString();
					})
				.OnObjectChanged_Lambda([this, InPickerArgs](const FAssetData& InAssetData)
					{
						SelectedAsset = InAssetData;

						TObjectPtr<UObject> NewSelectedAsset = InAssetData.GetSoftObjectPath().TryLoad();
						TObjectPtr<UBlendProfileStandalone> BlendProfile = Cast<UBlendProfileStandalone>(NewSelectedAsset);

						TObjectPtr<UBlendProfileStandaloneProvider> BlendProfileProvider = NewObject<UBlendProfileStandaloneProvider>(InPickerArgs.Outer);
						BlendProfileProvider->Initialize(BlendProfile);

						InPickerArgs.OnProviderChanged.ExecuteIfBound(BlendProfileProvider, BlendProfileProvider);
					})
				.AllowedClass(UBlendProfileStandalone::StaticClass())
				.OnShouldFilterAsset_Lambda([InPickerArgs](const FAssetData& InAssetData)
					{
						const FString BlendProfileTypeString = InAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UBlendProfileStandalone, Type));

						const EBlendProfileStandaloneType BlendProfileType = static_cast<EBlendProfileStandaloneType>(StaticEnum<EBlendProfileStandaloneType>()->GetValueByName(FName(*BlendProfileTypeString)));

						// Only show blend profile assets with the required type
						bool bKeep = false;
						switch (InPickerArgs.SupportedBlendProfileModes) 
						{
							case EBlendProfilePickerMode::AllModes:
								bKeep = true;
								break;
							case EBlendProfilePickerMode::BlendProfile:
								bKeep = (BlendProfileType == EBlendProfileStandaloneType::WeightFactor) || (BlendProfileType ==  EBlendProfileStandaloneType::TimeFactor);
								break;
							case EBlendProfilePickerMode::BlendMask:
								bKeep = (BlendProfileType == EBlendProfileStandaloneType::BlendMask);
								break;
						}

						if (bKeep)
						{
							// Only display blend profile assets with matching skeletons
							FString SkeletonSoftObjectPath;
							if (InAssetData.GetTagValue("Skeleton", SkeletonSoftObjectPath))
							{
								if (InPickerArgs.Skeleton && FAssetData(InPickerArgs.Skeleton).ToSoftObjectPath().ToString() != SkeletonSoftObjectPath)
								{
									bKeep = false;
								}
							}
						}

						return !bKeep;
					})
		];
}
