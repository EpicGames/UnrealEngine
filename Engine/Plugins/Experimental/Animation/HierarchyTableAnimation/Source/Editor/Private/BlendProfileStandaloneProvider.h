// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/BlendProfile.h"
#include "BlendProfileStandalone.h"
#include "IBlendProfilePickerExtender.h"
#include "UObject/NoExportTypes.h"
#include "Widgets/SCompoundWidget.h"

#include "BlendProfileStandaloneProvider.generated.h"

#define UE_API HIERARCHYTABLEANIMATIONEDITOR_API

/**
 * Class responsible for actually creating new blend profiles from custom data
 */
UCLASS(MinimalAPI)
class UBlendProfileStandaloneProvider : public UObject, public IBlendProfileProviderInterface
{
	GENERATED_BODY()

public:
	// IBlendProfileProviderInterface
	UE_API void Initialize(TObjectPtr<UBlendProfileStandalone> InBlendProfile);
	UE_API void ConstructBlendProfile(const TObjectPtr<UBlendProfile> OutBlendProfile) const override;

	UPROPERTY()
	TObjectPtr<UBlendProfileStandalone> BlendProfile;
};

/**
 * Class responsible for adding a new blend profile provider in the editor for FBlendProfileInterfaceWrapper
 */
class FBlendProfileStandalonePickerExtender : public IBlendProfilePickerExtender
{
public:
	static FName StaticGetId();

	// IBlendProfilePickerExtender
	FName GetId() const override;
	FText GetDisplayName() const override;
	TSharedRef<SWidget> ConstructPickerWidget(const FPickerWidgetArgs& InWidgetArgs) const override;
	bool OwnsBlendProfileProvider(const TObjectPtr<const UObject> InObject) const override;

private:
	class SPicker : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SPicker)
		{
		}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, IBlendProfilePickerExtender::FPickerWidgetArgs InPickerArgs);

	private:
		FAssetData SelectedAsset;
	};
};

#undef UE_API
