// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableAnimationEditorModule.h"
#include "MaskProfile/HierarchyTableTypeMask.h"
#include "Modules/ModuleManager.h"
#include "HierarchyTableEditorModule.h"
#include "SkeletonHierarchyTableType.h"
#include "SkeletonHierarchyTableTypeHandler.h"
#include "MaskProfile/MaskProfileColumn.h"
#include "SkeletonHierarchyTableTypeDetailsCustomization.h"
#include "PropertyEditorModule.h"
#include "PersonaModule.h"
#include "BlendProfileStandaloneProvider.h"

void FHierarchyTableAnimationEditorModule::StartupModule()
{
	FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::LoadModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");
	
	BuiltinTableTypes.Add(FHierarchyTable_TableType_Skeleton::StaticStruct());
	HierarchyTableModule.RegisterTableType(FHierarchyTable_TableType_Skeleton::StaticStruct(), UHierarchyTable_TableTypeHandler_Skeleton::StaticClass());

	BuiltinElementTypes.Add(FHierarchyTable_ElementType_Mask::StaticStruct());
	HierarchyTableModule.RegisterElementTypeEditorColumns(FHierarchyTable_ElementType_Mask::StaticStruct(),
		{
			MakeShared<FHierarchyTableColumn_Mask>(),
		});

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomPropertyTypeLayout(FHierarchyTable_TableType_Skeleton::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FHierarchyTableSkeletonTableTypeDetailsCustomization::MakeInstance));

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaModule.RegisterBlendProfilePickerExtender(MakeShared<FBlendProfileStandalonePickerExtender>());
}

void FHierarchyTableAnimationEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("HierarchyTableEditor"))
	{
		FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::GetModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");

		for (TWeakObjectPtr<UScriptStruct> WeakPtr : BuiltinTableTypes)
		{
			if (const UScriptStruct* StructPtr = WeakPtr.Get())
			{
				HierarchyTableModule.UnregisterTableType(StructPtr);
			}
		}

		for (TWeakObjectPtr<UScriptStruct> WeakPtr : BuiltinElementTypes)
		{
			if (const UScriptStruct* StructPtr = WeakPtr.Get())
			{
				HierarchyTableModule.UnregisterElementTypeEditorColumns(StructPtr);
			}
		}
	}
	
	if (FModuleManager::Get().IsModuleLoaded("Persona"))
	{
		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
		PersonaModule.UnregisterBlendProfilePickerExtender(FBlendProfileStandalonePickerExtender::StaticGetId());
	}
}

IMPLEMENT_MODULE(FHierarchyTableAnimationEditorModule, HierarchyTableAnimationEditor)
