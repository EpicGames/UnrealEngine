// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaskProfile/MaskProfileColumn.h"
#include "Widgets/Input/SSpinBox.h"
#include "MaskProfile/HierarchyTableTypeMask.h"
#include "HierarchyTable.h"
#include "Editor.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FHierarchyTableColumn_Mask"

TSharedRef<SWidget> FHierarchyTableColumn_Mask::CreateEntryWidget(TObjectPtr<UHierarchyTable> HierarchyTable, int32 EntryIndex)
{
	return SNew(SSpinBox<float>)
		.IsEnabled_Lambda([HierarchyTable, EntryIndex]() { return HierarchyTable->GetTableEntry(EntryIndex)->IsOverridden(); })
		.MinDesiredWidth(100.0f)
		.MinValue(0.0f)
		.MaxValue(1.0f)
		.Value_Lambda([HierarchyTable, EntryIndex]()
			{
				return HierarchyTable->GetTableEntry(EntryIndex)->GetValue<FHierarchyTable_ElementType_Mask>()->Value;
			})
		.OnValueChanged_Lambda([HierarchyTable, EntryIndex](const float NewValue)
			{
				const FScopedTransaction Transaction(LOCTEXT("ModifyValue", "Modify Value"));
				HierarchyTable->Modify();
			
				// TODO: Update API to avoid having to manually regenerate guid
				HierarchyTable->GetMutableTableEntry(EntryIndex)->GetMutableValue<FHierarchyTable_ElementType_Mask>()->Value = NewValue;
				HierarchyTable->RegenerateEntriesGuid();
			})
		.OnBeginSliderMovement_Lambda([HierarchyTable]()
			{
				GEditor->BeginTransaction(LOCTEXT("SetMaskValue", "Set Mask Value"));
				HierarchyTable->Modify();
			})
		.OnEndSliderMovement_Lambda([](float)
			{
				GEditor->EndTransaction();
			});
}

TSharedRef<SWidget> FHierarchyTableColumn_Mask::CreateHeaderWidget()
{
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE