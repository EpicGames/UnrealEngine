// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IHierarchyTableColumn.h"

#define LOCTEXT_NAMESPACE "HierarchyTableColumn_Mask"

struct FHierarchyTableColumn_Mask : public IHierarchyTableColumn
{
public:
	virtual FName GetColumnId() const override { return FName("Mask"); }
	virtual FText GetColumnLabel() const override { return LOCTEXT("MaskLabel", "Mask"); }
	virtual float GetColumnSize() const override { return 1.0f; }

	virtual TSharedRef<SWidget> CreateEntryWidget(TObjectPtr<UHierarchyTable> HierarchyTable, int32 EntryIndex);
	virtual TSharedRef<SWidget> CreateHeaderWidget();
};

#undef LOCTEXT_NAMESPACE