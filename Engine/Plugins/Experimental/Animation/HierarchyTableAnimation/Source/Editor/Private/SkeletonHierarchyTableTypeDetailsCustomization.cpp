// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonHierarchyTableTypeDetailsCustomization.h"
#include "DetailLayoutBuilder.h"
#include "HierarchyTable.h"
#include "HierarchyTableEditorModule.h"
#include "SkeletonHierarchyTableType.h"
#include "IDetailChildrenBuilder.h"
#include "Modules/ModuleManager.h"

TSharedRef<IPropertyTypeCustomization> FHierarchyTableSkeletonTableTypeDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FHierarchyTableSkeletonTableTypeDetailsCustomization());
}

void FHierarchyTableSkeletonTableTypeDetailsCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FHierarchyTableSkeletonTableTypeDetailsCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TArray<UObject*> EditingObjects;
	StructPropertyHandle->GetOuterObjects(EditingObjects);

	if (EditingObjects.Num() != 1)
	{
		return;
	}

	TObjectPtr<UHierarchyTable> HierarchyTable = CastChecked<UHierarchyTable>(EditingObjects[0]);

	if (StructPropertyHandle->IsValidHandle())
	{
		uint32 NumChildren = 0;

		StructPropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
		{
			TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();

			if (ChildHandle->IsValidHandle())
			{
				if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FHierarchyTable_TableType_Skeleton, Skeleton))
				{
					ChildHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([HierarchyTable]()
						{
							FHierarchyTableEditorModule& HierarchyTableModule = FModuleManager::LoadModuleChecked<FHierarchyTableEditorModule>("HierarchyTableEditor");
							const TObjectPtr<UHierarchyTable_TableTypeHandler> HierarchyTableHandler = HierarchyTableModule.CreateTableHandler(HierarchyTable);
							check(HierarchyTableHandler);

							HierarchyTableHandler->ConstructHierarchy();
						}));
				}

				StructBuilder.AddProperty(ChildHandle);
			}
		}
	}
}