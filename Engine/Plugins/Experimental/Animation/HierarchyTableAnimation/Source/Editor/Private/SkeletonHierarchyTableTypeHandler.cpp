// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonHierarchyTableTypeHandler.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SkeletonHierarchyTableType.h"
#include "Animation/Skeleton.h"
#include "Widgets/Layout/SBorder.h"
#include "Editor.h"
#include "StructUtils/InstancedStruct.h"
#include "ToolMenu.h"
#include "ToolMenuMisc.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "PersonaModule.h"
#include "HierarchyTable.h"
#include "HierarchyTableEditorModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletonHierarchyTableTypeHandler)

#define LOCTEXT_NAMESPACE "UHierarchyTable_TableTypeHandler_Skeleton"

FSlateIcon UHierarchyTable_TableTypeHandler_Skeleton::GetEntryIcon(const int32 EntryIndex) const
{
	const FHierarchyTableEntryData* const Entry = HierarchyTable->GetTableEntry(EntryIndex);
	const FHierarchyTable_TablePayloadType_Skeleton& EntryTablePayload = Entry->TablePayload.Get<FHierarchyTable_TablePayloadType_Skeleton>();

	switch (EntryTablePayload.EntryType)
	{
	case ESkeletonHierarchyTable_TablePayloadEntryType::Bone:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "SkeletonTree.Bone");
	case ESkeletonHierarchyTable_TablePayloadEntryType::Curve:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimGraph.Attribute.Curves.Icon");
	case ESkeletonHierarchyTable_TablePayloadEntryType::Attribute:
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimGraph.Attribute.Attributes.Icon");
	}

	return FSlateIcon();
}

FSlateColor UHierarchyTable_TableTypeHandler_Skeleton::GetEntryIconColor(const int32 EntryIndex) const
{
	const FHierarchyTableEntryData* const Entry = HierarchyTable->GetTableEntry(EntryIndex);
	const FHierarchyTable_TablePayloadType_Skeleton& EntryTablePayload = Entry->TablePayload.Get<FHierarchyTable_TablePayloadType_Skeleton>();

	switch (EntryTablePayload.EntryType)
	{
	case ESkeletonHierarchyTable_TablePayloadEntryType::Bone:
		return FSlateColor::UseForeground();
	case ESkeletonHierarchyTable_TablePayloadEntryType::Curve:
		return FAppStyle::GetSlateColor("AnimGraph.Attribute.Curves.Color");
	case ESkeletonHierarchyTable_TablePayloadEntryType::Attribute:
		return  FAppStyle::GetSlateColor("AnimGraph.Attribute.Attributes.Color");
	}

	return FSlateColor::UseForeground();
}

void UHierarchyTable_TableTypeHandler_Skeleton::ExtendContextMenu(FMenuBuilder& MenuBuilder, IHierarchyTable& HierarchyTableView) const
{
	const int32 SelectedIndex = HierarchyTableView.GetSelectedEntryIndex();

	const FHierarchyTableEntryData* EntryData = HierarchyTable->GetTableEntry(SelectedIndex);
	check(EntryData);

	const FHierarchyTable_TablePayloadType_Skeleton& EntryMetadata = EntryData->GetMetadata<FHierarchyTable_TablePayloadType_Skeleton>();

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

	const FHierarchyTable_TableType_Skeleton TableMetadata = HierarchyTable->GetTableMetadata<FHierarchyTable_TableType_Skeleton>();

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddCurve_Label", "Add Curve"),
		LOCTEXT("AddCurve_Tooltip", "Add a new curve entry parented to this entry"),
		FNewMenuDelegate::CreateLambda([this, SelectedIndex, &HierarchyTableView, &PersonaModule, TableMetadata](FMenuBuilder& InSubMenuBuilder)
			{
				InSubMenuBuilder.BeginSection(NAME_None, LOCTEXT("NewCurveSection", "New Curve"));
				{
					InSubMenuBuilder.AddWidget(
						SNew(SBox)
						.Padding(FMargin(8.0f, 0.0f))
						[
							SNew(SEditableTextBox)
								.Text(LOCTEXT("NewCurveDefault", "NewCurve"))
								.OnTextCommitted_Lambda([this, SelectedIndex, &HierarchyTableView](const FText& InText, ETextCommit::Type InTextCommit)
									{
										if (InTextCommit == ETextCommit::OnEnter)
										{
											const int32 ParentIndex = SelectedIndex == INDEX_NONE ? 0 : SelectedIndex;
											AddCurve(ParentIndex, FName(InText.ToString()));
										}
									})
								.OnVerifyTextChanged_Lambda([this](const FText& InNewText, FText& OutErrorMessage) -> bool
									{
										if (InNewText.IsEmpty())
										{
											OutErrorMessage = LOCTEXT("CurveNameEmpty", "Name can't be empty.");
											return false;
										}

										if (HierarchyTable->HasIdentifier(FName(InNewText.ToString())))
										{
											OutErrorMessage = LOCTEXT("CurveNameExists", "Name already exists in the hierarchy.");
											return false;
										}

										return true;
									})
						],
						FText::GetEmpty(), true);
				}
				InSubMenuBuilder.EndSection();

				InSubMenuBuilder.BeginSection(NAME_None, LOCTEXT("ExistingCurveSection", "Existing Curve"));
				{
					InSubMenuBuilder.AddWidget(
						PersonaModule.CreateMultiCurvePicker(TableMetadata.Skeleton,
							FOnCurvesPicked::CreateLambda([this, SelectedIndex, &HierarchyTableView](const TConstArrayView<FName> InCurves)
								{
									FSlateApplication::Get().DismissAllMenus();

									const int32 ParentIndex = SelectedIndex == INDEX_NONE ? 0 : SelectedIndex;
									for (FName Curve : InCurves)
									{
										AddCurve(ParentIndex, Curve);
									}
								}),
							FIsCurveNameMarkedForExclusion::CreateLambda([this](const FName& InName)
								{
									return HierarchyTable->HasIdentifier(InName);
								})),
						FText::GetEmpty(), true);
				}
				InSubMenuBuilder.EndSection();
			})
	);

	MenuBuilder.AddSubMenu(
		LOCTEXT("AddAttribute_Label", "Add Attribute"),
		LOCTEXT("AddAttribut_Tooltip", "Add a new attribute entry parented to this entry"),
		FNewMenuDelegate::CreateLambda([this, SelectedIndex, &HierarchyTableView, &PersonaModule, TableMetadata](FMenuBuilder& InSubMenuBuilder)
			{
				InSubMenuBuilder.BeginSection(NAME_None, LOCTEXT("NewAttributeSection", "New Attribute"));
				{
					InSubMenuBuilder.AddWidget(
						SNew(SBox)
						.Padding(FMargin(8.0f, 0.0f))
						[
							SNew(SEditableTextBox)
								.Text(LOCTEXT("NewAttributeDefault", "NewAttribute"))
								.OnTextCommitted_Lambda([this, SelectedIndex, &HierarchyTableView](const FText& InText, ETextCommit::Type InTextCommit)
									{
										if (InTextCommit == ETextCommit::OnEnter)
										{
											const int32 ParentIndex = SelectedIndex == INDEX_NONE ? 0 : SelectedIndex;
											AddAttribute(ParentIndex, FName(InText.ToString()));
										}
									})
								.OnVerifyTextChanged_Lambda([this](const FText& InNewText, FText& OutErrorMessage) -> bool
									{
										if (InNewText.IsEmpty())
										{
											OutErrorMessage = LOCTEXT("RenameEmpty", "Name can't be empty.");
											return false;
										}

										if (HierarchyTable->HasIdentifier(FName(InNewText.ToString())))
										{
											OutErrorMessage = LOCTEXT("RenameExists", "Name already exists in the hierarchy.");
											return false;
										}

										return true;
									})
						],
						FText::GetEmpty(), true);
				}
				InSubMenuBuilder.EndSection();
			})
	);
}

void UHierarchyTable_TableTypeHandler_Skeleton::ConstructHierarchy()
{
	HierarchyTable->EmptyTable();

	const FHierarchyTable_TableType_Skeleton& SkeletonTableType = HierarchyTable->GetTableMetadata<FHierarchyTable_TableType_Skeleton>();
	check(SkeletonTableType.Skeleton);

	FInstancedStruct DefaultEntry = HierarchyTable->CreateDefaultValue();

	FInstancedStruct DefaultTablePayload;
	DefaultTablePayload.InitializeAs<FHierarchyTable_TablePayloadType_Skeleton>();
	DefaultTablePayload.GetMutable<FHierarchyTable_TablePayloadType_Skeleton>().EntryType = ESkeletonHierarchyTable_TablePayloadEntryType::Bone;

	FReferenceSkeleton RefSkeleton = SkeletonTableType.Skeleton->GetReferenceSkeleton();

	TArray<FHierarchyTableEntryData> EntriesToAdd;

	for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
	{
		FHierarchyTableEntryData EntryData;
		EntryData.Parent = RefSkeleton.GetParentIndex(BoneIndex);
		EntryData.Identifier = RefSkeleton.GetBoneName(BoneIndex);
		EntryData.TablePayload = DefaultTablePayload;
		EntryData.Payload = BoneIndex == 0 ? DefaultEntry : TOptional<FInstancedStruct>();
		EntryData.OwnerTable = HierarchyTable;

		EntriesToAdd.Add(EntryData);
	}

	HierarchyTable->AddBulkEntries(EntriesToAdd);
}

bool UHierarchyTable_TableTypeHandler_Skeleton::FactoryConfigureProperties(FInstancedStruct& TableType) const
{
	FHierarchyTable_TableType_Skeleton& SkeletonTableType = TableType.GetMutable<FHierarchyTable_TableType_Skeleton>();

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	TSharedPtr<class SWindow> PickerWindow;

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([&](const FAssetData& SelectedAsset)
		{
			SkeletonTableType.Skeleton = Cast<USkeleton>(SelectedAsset.GetAsset());
			PickerWindow->RequestDestroyWindow();
		});

	PickerWindow = SNew(SWindow)
		.Title(INVTEXT("Pick Skeleton"))
		.ClientSize(FVector2D(500, 600))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]
		];

	GEditor->EditorAddModalWindow(PickerWindow.ToSharedRef());
	PickerWindow.Reset();

	return SkeletonTableType.Skeleton != nullptr;
}

void UHierarchyTable_TableTypeHandler_Skeleton::ExtendToolbar(UToolMenu* ToolMenu, IHierarchyTable& HierarchyTableView) const
{
}

void UHierarchyTable_TableTypeHandler_Skeleton::AddCurve(const int32 ParentIndex, const FName Identifier) const
{
	FScopedTransaction AddCurveTransaction(LOCTEXT("AddCurveTransaction", "Add Curve Entry"));
	HierarchyTable->Modify();

	FHierarchyTableEntryData NewEntry;
	NewEntry.Parent = ParentIndex;
	NewEntry.Identifier = Identifier;
	NewEntry.Payload = TOptional<FInstancedStruct>();
	NewEntry.OwnerTable = HierarchyTable;
	{
		FInstancedStruct TablePayload;
		TablePayload.InitializeAs<FHierarchyTable_TablePayloadType_Skeleton>();
		FHierarchyTable_TablePayloadType_Skeleton& SkeletonTablePayload = TablePayload.GetMutable<FHierarchyTable_TablePayloadType_Skeleton>();
		SkeletonTablePayload.EntryType = ESkeletonHierarchyTable_TablePayloadEntryType::Curve;
		
		NewEntry.TablePayload = TablePayload;
	}

	HierarchyTable->AddEntry(NewEntry);
}
	
void UHierarchyTable_TableTypeHandler_Skeleton::AddAttribute(const int32 ParentIndex, const FName Identifier) const
{
	FScopedTransaction AddAttributeTransaction(LOCTEXT("AddAttributeTransaction", "Add Attribute Entry"));
	HierarchyTable->Modify();

	FHierarchyTableEntryData NewEntry;
	NewEntry.Parent = ParentIndex;
	NewEntry.Identifier = Identifier;
	NewEntry.Payload = TOptional<FInstancedStruct>();
	NewEntry.OwnerTable = HierarchyTable;
	{
		FInstancedStruct TablePayload;
		TablePayload.InitializeAs<FHierarchyTable_TablePayloadType_Skeleton>();
		FHierarchyTable_TablePayloadType_Skeleton& SkeletonTablePayload = TablePayload.GetMutable<FHierarchyTable_TablePayloadType_Skeleton>();
		SkeletonTablePayload.EntryType = ESkeletonHierarchyTable_TablePayloadEntryType::Attribute;
		
		NewEntry.TablePayload = TablePayload;
	}

	HierarchyTable->AddEntry(NewEntry);
}

bool UHierarchyTable_TableTypeHandler_Skeleton::CanRenameEntry(const int32 EntryIndex) const
{
	if (const FHierarchyTableEntryData* const Entry = HierarchyTable->GetTableEntry(EntryIndex))
	{
		const FHierarchyTable_TablePayloadType_Skeleton& EntryMetadata = Entry->GetMetadata<FHierarchyTable_TablePayloadType_Skeleton>();
		const ESkeletonHierarchyTable_TablePayloadEntryType EntryType = EntryMetadata.EntryType;

		return EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Curve || EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Attribute;
	}

	return false;
}

bool UHierarchyTable_TableTypeHandler_Skeleton::CanRemoveEntry(const int32 EntryIndex) const
{
	if (const FHierarchyTableEntryData* const Entry = HierarchyTable->GetTableEntry(EntryIndex))
	{
		const FHierarchyTable_TablePayloadType_Skeleton& EntryMetadata = Entry->GetMetadata<FHierarchyTable_TablePayloadType_Skeleton>();
		const ESkeletonHierarchyTable_TablePayloadEntryType EntryType = EntryMetadata.EntryType;

		return EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Curve || EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Attribute;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
