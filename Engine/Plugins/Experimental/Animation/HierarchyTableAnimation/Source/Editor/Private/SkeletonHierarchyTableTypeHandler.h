// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyTableTypeHandler.h"

#include "SkeletonHierarchyTableTypeHandler.generated.h"

UCLASS()
class UHierarchyTable_TableTypeHandler_Skeleton final : public UHierarchyTable_TableTypeHandler
{
	GENERATED_BODY()

public:
	virtual void ConstructHierarchy();

	virtual bool FactoryConfigureProperties(FInstancedStruct& TableType) const override;

	virtual void ExtendToolbar(UToolMenu* ToolMenu, IHierarchyTable& HierarchyTableView) const override;

	virtual void ExtendContextMenu(FMenuBuilder& MenuBuilder, IHierarchyTable& HierarchyTableView) const override;

	virtual FSlateIcon GetEntryIcon(const int32 EntryIndex) const override;

	virtual FSlateColor GetEntryIconColor(const int32 EntryIndex) const override;

	virtual bool CanRenameEntry(const int32 EntryIndex) const override;

	virtual bool CanRemoveEntry(const int32 EntryIndex) const override;

private:
	void AddCurve(const int32 ParentIndex, const FName Identifier) const;

	void AddAttribute(const int32 ParentIndex, const FName Identifier) const;
};