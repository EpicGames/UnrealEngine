// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class HierarchyTableAnimationRuntime : ModuleRules
{
	public HierarchyTableAnimationRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "HTAnimRun";

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"HierarchyTableRuntime"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
            {
			}
		);
	}
}
