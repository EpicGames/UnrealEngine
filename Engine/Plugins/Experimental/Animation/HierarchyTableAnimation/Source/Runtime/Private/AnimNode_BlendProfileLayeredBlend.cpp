// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_BlendProfileLayeredBlend.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimCurveUtils.h"
#include "Animation/AttributeTypes.h"
#include "Animation/IAttributeBlendOperator.h"
#include "HierarchyTableBlendProfile.h"
#include "BlendProfileStandalone.h"
#include "Animation/AnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_BlendProfileLayeredBlend)

void FAnimNode_BlendProfileLayeredBlend::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	FAnimNode_Base::Initialize_AnyThread(Context);

	BasePose.Initialize(Context);
	BlendPose.Initialize(Context);
}

bool FAnimNode_BlendProfileLayeredBlend::ArePerBoneBlendWeightsValid(const USkeleton* InSkeleton) const
{
	return InSkeleton != nullptr
		&& InSkeleton->GetGuid() == SkeletonGuid
		&& InSkeleton->GetVirtualBoneGuid() == VirtualBoneGuid
		&& CachedBlendProfile == BlendProfileAsset
#if WITH_EDITOR
		/* Support editor-only live updating blend weights when the blend profile values change */
		&& (BlendProfileAsset && !BlendProfileAsset->IsCachedDataStale());
#else
		;
#endif
}
void FAnimNode_BlendProfileLayeredBlend::UpdateCachedBoneData(const FBoneContainer& RequiredBones, USkeleton* Skeleton)
{
	if (ArePerBoneBlendWeightsValid(Skeleton) && RequiredBonesSerialNumber == RequiredBones.GetSerialNumber())
	{
		return;
	}
	
#if WITH_EDITOR
	if (BlendProfileAsset && BlendProfileAsset->IsCachedDataStale())
	{
		BlendProfileAsset->UpdateCachedData();
	}
#endif // WITH_EDITOR

	QUICK_SCOPE_CYCLE_COUNTER(STAT_BlendProfileLayeredBlend_UpdateCachedBoneData);
	
	const TArray<FBoneIndexType>& RequiredBoneIndices = RequiredBones.GetBoneIndicesArray();
	const int32 NumRequiredBones = RequiredBoneIndices.Num();

	// Update DesiredBoneBlendWeights
	const bool bBlendProfileValid = Skeleton && BlendProfileAsset && BlendProfileAsset->Type == EBlendProfileStandaloneType::BlendMask && BlendProfileAsset->GetSkeleton() == Skeleton;
	if (bBlendProfileValid)
	{
		if (ensure(NumRequiredBones <= BlendProfileAsset->CachedBlendProfileData.GetBoneBlendWeights().Num()))
		{
			DesiredBoneBlendWeights.SetNumUninitialized(NumRequiredBones);
			const TArray<float>& BoneBlendWeights = BlendProfileAsset->CachedBlendProfileData.GetBoneBlendWeights();
			for (int32 RequiredBoneIndex = 0; RequiredBoneIndex < NumRequiredBones; ++RequiredBoneIndex)
			{
				const int32 SkeletonBoneIndex = RequiredBones.GetSkeletonIndex(FCompactPoseBoneIndex(RequiredBoneIndex));
				DesiredBoneBlendWeights[RequiredBoneIndex] = BoneBlendWeights[SkeletonBoneIndex];
			}
		}
	}

	// Update CurrentBoneBlendWeights
	{
		CurrentBoneBlendWeights.Reset(DesiredBoneBlendWeights.Num());
		CurrentBoneBlendWeights.AddZeroed(DesiredBoneBlendWeights.Num());
		
		//Reinitialize bone blend weights now that we have cleared them
		UpdateDesiredBoneWeight();
	}

	SkeletonGuid = Skeleton ? Skeleton->GetGuid() : FGuid();
	VirtualBoneGuid = Skeleton ? Skeleton->GetVirtualBoneGuid() : FGuid();
	RequiredBonesSerialNumber = RequiredBones.GetSerialNumber();
	CachedBlendProfile = BlendProfileAsset;
}

void FAnimNode_BlendProfileLayeredBlend::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	
	BasePose.CacheBones(Context);
	BlendPose.CacheBones(Context);

	UpdateCachedBoneData(Context.AnimInstanceProxy->GetRequiredBones(), Context.AnimInstanceProxy->GetSkeleton());
}

void FAnimNode_BlendProfileLayeredBlend::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)

	bool RootMotionBlendPose = false;
	float RootMotionWeight = 0.f;
	const float RootMotionClearWeight = bBlendRootMotionBasedOnRootBone ? 0.f : 1.f;

	if (IsLODEnabled(Context.AnimInstanceProxy))
	{
		GetEvaluateGraphExposedInputs().Execute(Context);

		if (FAnimWeight::IsRelevant(BlendWeight))
		{
			UpdateCachedBoneData(Context.AnimInstanceProxy->GetRequiredBones(), Context.AnimInstanceProxy->GetSkeleton());
			UpdateDesiredBoneWeight();

			if (bBlendRootMotionBasedOnRootBone && !CurrentBoneBlendWeights.IsEmpty())
			{
				const float NewRootMotionWeight = CurrentBoneBlendWeights[0];
				if (NewRootMotionWeight > ZERO_ANIMWEIGHT_THRESH)
				{
					RootMotionWeight = NewRootMotionWeight;
					RootMotionBlendPose = true;
				}
			}

			const float ThisPoseRootMotionWeight = RootMotionBlendPose ? RootMotionWeight : RootMotionClearWeight;
			BlendPose.Update(Context.FractionalWeightAndRootMotion(BlendWeight, ThisPoseRootMotionWeight));
		}
	}

	// initialize children
	const float BaseRootMotionWeight = 1.f - RootMotionWeight;

	if (BaseRootMotionWeight < ZERO_ANIMWEIGHT_THRESH)
	{
		BasePose.Update(Context.FractionalWeightAndRootMotion(1.f, BaseRootMotionWeight));
	}
	else
	{
		BasePose.Update(Context);
	}
}

void FAnimNode_BlendProfileLayeredBlend::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER(BlendPosesInGraph, !IsInGameThread());

	TObjectPtr<USkeleton> Skeleton = Output.AnimInstanceProxy->GetSkeleton();
	const bool bBlendProfileValid = Skeleton && BlendProfileAsset && BlendProfileAsset->Type == EBlendProfileStandaloneType::BlendMask && BlendProfileAsset->GetSkeleton() == Skeleton;

	if (!FAnimWeight::IsRelevant(BlendWeight) || !bBlendProfileValid)
	{
		BasePose.Evaluate(Output);
		return;
	}

	FPoseContext BasePoseContext(Output);
	FPoseContext BlendPoseContext(Output);

	BasePose.Evaluate(BasePoseContext);
	BlendPose.Evaluate(BlendPoseContext);

	FAnimationPoseData BasePoseData(BasePoseContext);
	FAnimationPoseData BlendPoseData(BlendPoseContext);

	// Blend poses
	{
		// Root space blending: zero the root bone rotation before converting to mesh space so that
		// child rotations accumulate relative to root's local frame rather than full mesh space.
		// The root bone's own rotation is saved, blended separately in local space, and restored after.
		FQuat BaseRootRot = FQuat::Identity;
		FQuat TargetRootRot = FQuat::Identity;
		const FCompactPoseBoneIndex RootBoneIndex(0);
		if (RotationBlendSpace == EBlendProfileRotationBlendSpace::RootSpace)
		{
			BaseRootRot = BasePoseData.GetPose()[RootBoneIndex].GetRotation();
			TargetRootRot = BlendPoseData.GetPose()[RootBoneIndex].GetRotation();
			BasePoseData.GetPose()[RootBoneIndex].SetRotation(FQuat::Identity);
			BlendPoseData.GetPose()[RootBoneIndex].SetRotation(FQuat::Identity);
		}

		if (RotationBlendSpace != EBlendProfileRotationBlendSpace::LocalSpace)
		{
			FAnimationRuntime::ConvertPoseToMeshRotation(BasePoseData.GetPose());
			FAnimationRuntime::ConvertPoseToMeshRotation(BlendPoseData.GetPose());
		}

		FAnimationRuntime::BlendTwoPosesTogetherPerBone(BasePoseData.GetPose(), BlendPoseData.GetPose(), CurrentBoneBlendWeights, Output.Pose);

		if (RotationBlendSpace != EBlendProfileRotationBlendSpace::LocalSpace)
		{
			FAnimationRuntime::ConvertMeshRotationPoseToLocalSpace(Output.Pose);
		}

		if (RotationBlendSpace == EBlendProfileRotationBlendSpace::RootSpace)
		{
			const float RootBlendWeight = FMath::Clamp(CurrentBoneBlendWeights[0], 0.f, 1.f);
			FQuat BlendedRootRot;
			if (!FAnimWeight::IsRelevant(RootBlendWeight))
			{
				BlendedRootRot = BaseRootRot;
			}
			else if (FAnimWeight::IsFullWeight(RootBlendWeight))
			{
				BlendedRootRot = TargetRootRot;
			}
			else
			{
				BlendedRootRot = FQuat::FastLerp(BaseRootRot, TargetRootRot, RootBlendWeight);
				BlendedRootRot.Normalize();
			}
			Output.Pose[RootBoneIndex].SetRotation(BlendedRootRot);
		}
	}


	// Blend curves
	if (!bCustomCurveBlending)
	{
		const UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement>& CachedCurveMaskWeights = BlendProfileAsset->CachedBlendProfileData.GetCurveBlendWeights();

		Output.Curve.CopyFrom(BasePoseData.GetCurve());

		if (FAnimWeight::IsRelevant(BlendWeight))
		{
			FBlendedCurve FilteredCurves;

			// Multiply per-curve blend weights by matching blend pose curves
			UE::Anim::FNamedValueArrayUtils::Intersection(
				BlendPoseData.GetCurve(),
				CachedCurveMaskWeights,
				[this, &FilteredCurves](const UE::Anim::FCurveElement& InBlendElement, const UE::Anim::FCurveElement& InMaskElement) mutable
					{
						FilteredCurves.Add(InBlendElement.Name, InBlendElement.Value * InMaskElement.Value);
					});

			// Override blend curve values with premultipled curves
			BlendPoseData.GetCurve().Combine(FilteredCurves);

			// Remove curves that have been filtered by the mask, curves with no mask value defined remain, even with 0.0 value
			UE::Anim::FNamedValueArrayUtils::RemoveByPredicate(
				BlendPoseData.GetCurve(),
				CachedCurveMaskWeights,
				[](const UE::Anim::FCurveElement& InBaseElement, const UE::Anim::FCurveElement& InMaskElement)
				{	
					const bool bKeep = InMaskElement.Value == 0.0;
					return bKeep;
				}
			);

			// Combine base and filtered pre-multiplied blend curves
			UE::Anim::FNamedValueArrayUtils::Union(
				Output.Curve,
				BlendPoseData.GetCurve(),
				[this](UE::Anim::FCurveElement& InOutBaseElement, const UE::Anim::FCurveElement& InBlendElement, UE::Anim::ENamedValueUnionFlags InFlags)
					{
						if (InFlags == UE::Anim::ENamedValueUnionFlags::BothArgsValid || InFlags == UE::Anim::ENamedValueUnionFlags::ValidArg1)
						{
							InOutBaseElement.Value = FMath::Lerp(InOutBaseElement.Value, InBlendElement.Value, BlendWeight);
							InOutBaseElement.Flags |= InBlendElement.Flags; // Should this only apply when the weight is relevant?
						}
					});
		}
	}
	else
	{
		float TargetPoseMaxWeight = 0.0f;

		for (const FCompactPoseBoneIndex BoneIndex : BlendPoseData.GetPose().ForEachBoneIndex())
		{
			const float BoneBlendWeight = FMath::Clamp(CurrentBoneBlendWeights[BoneIndex.GetInt()], 0.f, 1.f);
			TargetPoseMaxWeight = FMath::Max(TargetPoseMaxWeight, BoneBlendWeight);
		}

		const FBlendedCurve* SourceCurves[] = { &BasePoseData.GetCurve(), &BlendPoseData.GetCurve() };
		const float SourceWeights[] = { BlendWeight, TargetPoseMaxWeight };

		BlendCurves(SourceCurves, SourceWeights, Output.Curve, CurveBlendingOption);
	}

	// Blend attributes
	{
		const TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight>& CachedAttributeMaskWeights = BlendProfileAsset->CachedBlendProfileData.GetAttributeBlendWeights();
		
		using namespace UE::Anim;

		FStackAttributeContainer OutputAttributes;

		// Attributes are to be masked out according to the mask weights in AttributeMaskWeights, if an attribute has no mask weight set then it
		// inherits the weight of whatever bone it is attached to. Below are possible configurations that we need to account for:

		// Root 0.0						Root set to 0.0 therefore a RootMotionDelta attribute will also be masked out without having to set an explicit entry in AttributeMaskWeights

		// Root 0.0						RootMotionDelta is set to 1.0 in AttributeMaskWeights despite the parent bone being masked out.
		//  \ RootMotionDelta 1.0

		// Root 1.0						RootMotionDelta is being masked out in AttributeMaskWeights despite the parent bone being kept.
		//  \ RootMotionDelta 0.0


		// Below is a table of the possible permutations of base/blend attributes being present/absent along with the possible mask values
		// k denotes some value in the range (0, 1) exclusive.
		// - denotes an absent attribute

		// Base | Blend | Weight | Output
		// ------------------------------
		// a    | b     | 1.0    | b
		// a    | b     | k      | lerp(a, b, k)
		// a    | b     | 0.0    | a
		// - - - - - - - - - - - - - - -
		// a    | -     | 1.0    | a
		// a    | -     | k      | a
		// a    | -     | 0.0    | a
		// - - - - - - - - - - - - - - -
		// -    | b     | 1.0    | b
		// -    | b     | k      | lerp(default, b, k)
		// -    | b     | 0.0    | -

		// 1. Blend attributes according to the bone blend weights,
		// i.e. an attribute's weight is determined by the weight of its attached bone.
		UE::Anim::Attributes::BlendAttributesPerBone(BasePoseData.GetAttributes(), BlendPoseData.GetAttributes(), CurrentBoneBlendWeights, OutputAttributes);

		// 2. For each attribute that has a custom weight, i.e. one's that shouldn't be weighted by its attached bone, go and correct the blended value.
		for (const FBlendProfileStandaloneCachedData::FMaskedAttributeWeight& MaskedAttribute : CachedAttributeMaskWeights)
		{
			const TConstArrayView<TWeakObjectPtr<UScriptStruct>> UniqueTypes = OutputAttributes.GetUniqueTypes();

			for (int32 UniqueTypeIndex = 0; UniqueTypeIndex < UniqueTypes.Num(); ++UniqueTypeIndex)
			{
				const TConstArrayView<FAttributeId> AttributeKeys = OutputAttributes.GetKeys(UniqueTypeIndex);
				const TConstArrayView<TWrappedAttribute<FAnimStackAllocator>> AttributeValues = OutputAttributes.GetValues(UniqueTypeIndex);
				
				const TWeakObjectPtr<UScriptStruct> AttributeType = UniqueTypes[UniqueTypeIndex];
				const IAttributeBlendOperator* Operator = UE::Anim::AttributeTypes::GetTypeOperator(AttributeType);

				TWrappedAttribute<FAnimStackAllocator> DefaultData(AttributeType.Get());
				AttributeType->InitializeStruct(DefaultData.GetPtr<void>());

				uint8* OutputData = OutputAttributes.Find(AttributeType.Get(), MaskedAttribute.Attribute);
				if (OutputData)
				{
					uint8* BaseData = BasePoseData.GetAttributes().Find(AttributeType.Get(), MaskedAttribute.Attribute);
					uint8* BlendData = BlendPoseData.GetAttributes().Find(AttributeType.Get(), MaskedAttribute.Attribute);

					if (BaseData && BlendData)
					{
						// Base | Blend | Weight | Output
						// ------------------------------
						// a    | b     | 1.0    | b
						// a    | b     | k      | lerp(a, b, k)
						// a    | b     | 0.0    | a

						Operator->Interpolate(BaseData, BlendData, MaskedAttribute.Weight, OutputData);
					}
					else if (BaseData && !BlendData)
					{
						// Base | Blend | Weight | Output
						// ------------------------------
						// a    | -     | 1.0    | a
						// a    | -     | k      | a
						// a    | -     | 0.0    | a

						*OutputData = *BaseData;
					}
					else if (!BaseData && BlendData)
					{
						// Base | Blend | Weight | Output
						// ------------------------------
						// -    | b     | 1.0    | b
						// -    | b     | k      | lerp(default, b, k)
						// -    | b     | 0.0    | -

						if (MaskedAttribute.Weight != 0.0f)
						{
							Operator->Interpolate(DefaultData.GetPtr<void>(), BlendData, MaskedAttribute.Weight, OutputData);
							
						}
						else
						{
							OutputAttributes.Remove(AttributeType.Get(), MaskedAttribute.Attribute);
						}
					}
				}				
			}
		}

		Output.CustomAttributes.MoveFrom(OutputAttributes);
	}
}

void FAnimNode_BlendProfileLayeredBlend::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)

	TObjectPtr<USkeleton> Skeleton = DebugData.AnimInstance->CurrentSkeleton;
	const bool bBlendProfileValid = Skeleton && BlendProfileAsset && BlendProfileAsset->Type == EBlendProfileStandaloneType::BlendMask && BlendProfileAsset->GetSkeleton() == Skeleton;

	BasePose.GatherDebugData(DebugData.BranchFlow(1.f));
	BlendPose.GatherDebugData(DebugData.BranchFlow(bBlendProfileValid ? BlendWeight : 0.0f));
}

void FAnimNode_BlendProfileLayeredBlend::UpdateDesiredBoneWeight()
{
	check(CurrentBoneBlendWeights.Num() == DesiredBoneBlendWeights.Num());

	CurrentBoneBlendWeights.Init(0, CurrentBoneBlendWeights.Num());

	for (int32 BoneIndex = 0; BoneIndex < DesiredBoneBlendWeights.Num(); ++BoneIndex)
	{
		float TargetBlendWeight = BlendWeight * DesiredBoneBlendWeights[BoneIndex];
		
		if (FAnimWeight::IsRelevant(TargetBlendWeight))
		{
			CurrentBoneBlendWeights[BoneIndex] = TargetBlendWeight;
		}
	}
}
