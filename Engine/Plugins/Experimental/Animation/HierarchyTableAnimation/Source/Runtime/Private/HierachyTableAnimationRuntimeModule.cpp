// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableAnimationRuntimeModule.h"
#include "Modules/ModuleManager.h"

void FHierarchyTableAnimationRuntimeModule::StartupModule()
{
}

void FHierarchyTableAnimationRuntimeModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FHierarchyTableAnimationRuntimeModule, HierarchyTableAnimationRuntime)
