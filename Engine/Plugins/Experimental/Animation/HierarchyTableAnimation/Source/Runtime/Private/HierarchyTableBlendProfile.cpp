// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableBlendProfile.h"

#include "SkeletonHierarchyTableType.h"
#include "HierarchyTableDefaultTypes.h"
#include "MaskProfile/HierarchyTableTypeMask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HierarchyTableBlendProfile)

void FBlendProfileStandaloneCachedData::UnpackCachedData()
{
	CurveBlendWeights.Reserve(SerializedCurveBlendWeights.Num());
	for (const FMaskedCurveWeightSerialised SerializedCurve : SerializedCurveBlendWeights)
	{
		CurveBlendWeights.Add(SerializedCurve.CurveName, SerializedCurve.Weight);
	}

	AttributeBlendWeights.Reserve(SerializedAttributeBlendWeights.Num());
	for (const FMaskedAttributeWeightSerialised& SerializedAttribute : SerializedAttributeBlendWeights)
	{
		UE::Anim::FAttributeId AttributeId(SerializedAttribute.AttributeName, SerializedAttribute.AttributeIndex, SerializedAttribute.AttributeNamespace);
		AttributeBlendWeights.Add(FMaskedAttributeWeight(AttributeId, SerializedAttribute.Weight));
	}
}

void FBlendProfileStandaloneCachedData::Init(TObjectPtr<UHierarchyTable> InHierarchyTable, const EBlendProfileMode InMode)
{
	Mode = InMode;

	const FHierarchyTable_TableType_Skeleton& TableMetadata = InHierarchyTable->GetTableMetadata<FHierarchyTable_TableType_Skeleton>();
	Skeleton = TableMetadata.Skeleton;

	const FReferenceSkeleton RefSkeleton = Skeleton->GetReferenceSkeleton();

	const int32 BoneCount = RefSkeleton.GetNum();
	BoneBlendWeights.Reserve(BoneCount);

	if (InHierarchyTable->IsElementType<FHierarchyTable_ElementType_Float>())
	{
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			if (const FHierarchyTableEntryData* EntryData = InHierarchyTable->GetTableEntry(BoneIndex))
			{
				BoneBlendWeights.Add(EntryData->GetValue<FHierarchyTable_ElementType_Float>()->Value);
			}
			else
			{
				BoneBlendWeights.Add(0.0f);
			}
		}
	}
	else if (InHierarchyTable->IsElementType<FHierarchyTable_ElementType_Mask>())
	{
		// Update bone weights
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			if (const FHierarchyTableEntryData* EntryData = InHierarchyTable->GetTableEntry(BoneIndex))
			{
				BoneBlendWeights.Add(EntryData->GetValue<FHierarchyTable_ElementType_Mask>()->Value);
			}
			else
			{
				BoneBlendWeights.Add(0.0f);
			}
		}

		// Update curve and attribute weights
		{
			CurveBlendWeights.Empty();
			AttributeBlendWeights.Empty();

			for (const FHierarchyTableEntryData& Entry : InHierarchyTable->GetTableData())
			{
				if (Entry.TablePayload.Get<FHierarchyTable_TablePayloadType_Skeleton>().EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Curve)
				{
					const float EntryWeight = Entry.GetValue<FHierarchyTable_ElementType_Mask>()->Value;
					CurveBlendWeights.Add<UE::Anim::FCurveElement>({ Entry.Identifier, EntryWeight });
				}
				else if (Entry.TablePayload.Get<FHierarchyTable_TablePayloadType_Skeleton>().EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Attribute)
				{
					const FHierarchyTableEntryData* ParentEntry = InHierarchyTable->GetTableEntry(Entry.Parent);
					check(ParentEntry && ParentEntry->GetMetadata<FHierarchyTable_TablePayloadType_Skeleton>().EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Bone);

					UE::Anim::FAttributeId Attribute(Entry.Identifier, Entry.Parent, TEXT("bone"));

					const float EntryWeight = Entry.GetValue<FHierarchyTable_ElementType_Mask>()->Value;
					AttributeBlendWeights.Add(FMaskedAttributeWeight(Attribute, EntryWeight));
				}
			}
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Unsupported hierarchy table type, use Float or Mask element type instead"));
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			BoneBlendWeights.Add(1.0f);
		}
	}
}

TObjectPtr<USkeleton> FBlendProfileStandaloneCachedData::GetSkeleton() const
{
	return Skeleton;
}

const TArray<float>& FBlendProfileStandaloneCachedData::GetBoneBlendWeights() const
{
	return BoneBlendWeights;
}

const UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement>& FBlendProfileStandaloneCachedData::GetCurveBlendWeights() const
{
	return CurveBlendWeights;
}

const TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight>& FBlendProfileStandaloneCachedData::GetAttributeBlendWeights() const
{
	return AttributeBlendWeights;
}

void FBlendProfileStandaloneCachedData::ConstructBlendProfile(const TObjectPtr<UBlendProfile> OutBlendProfile) const
{
	OutBlendProfile->SetSkeleton(Skeleton);
	OutBlendProfile->Mode = Mode;

	for (int32 BoneIndex = 0; BoneIndex < BoneBlendWeights.Num(); ++BoneIndex)
	{
		const float BlendWeight = BoneBlendWeights[BoneIndex];

		OutBlendProfile->SetBoneBlendScale(BoneIndex, BlendWeight, false /* bRecurse */, true /* bCreate */);
	}
}

void FBlendProfileStandaloneCachedData::Reset()
{
	Skeleton = nullptr;
	BoneBlendWeights.Empty();
	CurveBlendWeights.Empty();
	AttributeBlendWeights.Empty();
}

bool FBlendProfileStandaloneCachedData::IsValidBoneIndex(const int32 BoneIndex) const
{
	return BoneBlendWeights.IsValidIndex(BoneIndex);
}



FHierarchyTableBlendProfile::FHierarchyTableBlendProfile(TObjectPtr<UHierarchyTable> InHierarchyTable, const EBlendProfileMode InMode)
	: Mode(InMode)
{
	const FHierarchyTable_TableType_Skeleton& TableMetadata = InHierarchyTable->GetTableMetadata<FHierarchyTable_TableType_Skeleton>();
	Skeleton = TableMetadata.Skeleton;

	const FReferenceSkeleton RefSkeleton = Skeleton->GetReferenceSkeleton();

	const int32 BoneCount = RefSkeleton.GetNum();
	BoneBlendWeights.Reserve(BoneCount);

	if (InHierarchyTable->IsElementType<FHierarchyTable_ElementType_Float>())
	{
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			if (const FHierarchyTableEntryData* EntryData = InHierarchyTable->GetTableEntry(BoneIndex))
			{
				BoneBlendWeights.Add(EntryData->GetValue<FHierarchyTable_ElementType_Float>()->Value);
			}
			else
			{
				BoneBlendWeights.Add(0.0f);
			}
		}
	}
	else if (InHierarchyTable->IsElementType<FHierarchyTable_ElementType_Mask>())
	{
		// Update bone weights
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			if (const FHierarchyTableEntryData* EntryData = InHierarchyTable->GetTableEntry(BoneIndex))
			{
				BoneBlendWeights.Add(EntryData->GetValue<FHierarchyTable_ElementType_Mask>()->Value);
			}
			else
			{
				BoneBlendWeights.Add(0.0f);
			}
		}

		// Update curve and attribute weights
		{
			CurveBlendWeights.Empty();
			AttributeBlendWeights.Empty();

			for (const FHierarchyTableEntryData& Entry : InHierarchyTable->GetTableData())
			{
				if (Entry.TablePayload.Get<FHierarchyTable_TablePayloadType_Skeleton>().EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Curve)
				{
					const float EntryWeight = Entry.GetValue<FHierarchyTable_ElementType_Mask>()->Value;
					CurveBlendWeights.Add<UE::Anim::FCurveElement>({ Entry.Identifier, EntryWeight });
				}
				else if (Entry.TablePayload.Get<FHierarchyTable_TablePayloadType_Skeleton>().EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Attribute)
				{
					const FHierarchyTableEntryData* ParentEntry = InHierarchyTable->GetTableEntry(Entry.Parent);
					check(ParentEntry && ParentEntry->GetMetadata<FHierarchyTable_TablePayloadType_Skeleton>().EntryType == ESkeletonHierarchyTable_TablePayloadEntryType::Bone);

					UE::Anim::FAttributeId Attribute(Entry.Identifier, Entry.Parent, TEXT("bone"));

					const float EntryWeight = Entry.GetValue<FHierarchyTable_ElementType_Mask>()->Value;
					AttributeBlendWeights.Add(FBlendProfileStandaloneCachedData::FMaskedAttributeWeight(Attribute, EntryWeight));
				}
			}
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Unsupported hierarchy table type, use Float or Mask element type instead"));
		for (int32 BoneIndex = 0; BoneIndex < BoneCount; ++BoneIndex)
		{
			BoneBlendWeights.Add(1.0f);
		}
	}
}

FHierarchyTableBlendProfile::FHierarchyTableBlendProfile(const TObjectPtr<USkeleton> InSkeleton, const EBlendProfileMode InMode)
	: Mode(InMode)
{
	if (InSkeleton)
	{
		Skeleton = InSkeleton;
		const int32 BoneCount = Skeleton->GetReferenceSkeleton().GetNum();
		BoneBlendWeights.SetNumZeroed(BoneCount);
	}
	else
	{
		Skeleton = nullptr;
		BoneBlendWeights.SetNumZeroed(0);
	}
}

float FHierarchyTableBlendProfile::GetBoneBlendScale(int32 InBoneIdx) const
{
	return BoneBlendWeights[InBoneIdx];
}

int32 FHierarchyTableBlendProfile::GetNumBlendEntries() const
{
	return BoneBlendWeights.Num();
}

int32 FHierarchyTableBlendProfile::GetPerBoneInterpolationIndex(const FCompactPoseBoneIndex& InCompactPoseBoneIndex, const FBoneContainer& BoneContainer, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const
{
	return InCompactPoseBoneIndex.GetInt();
}

int32 FHierarchyTableBlendProfile::GetPerBoneInterpolationIndex(const FSkeletonPoseBoneIndex InSkeletonBoneIndex, const USkeleton* TargetSkeleton, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const
{
	return InSkeletonBoneIndex.GetInt();
}

TObjectPtr<USkeleton> FHierarchyTableBlendProfile::GetSkeleton() const
{
	return Skeleton;
}

const UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement>& FHierarchyTableBlendProfile::GetCurveBlendWeights() const
{
	return CurveBlendWeights;
}

const TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight>& FHierarchyTableBlendProfile::GetAttributeBlendWeights() const
{
	return AttributeBlendWeights;
}

EBlendProfileMode FHierarchyTableBlendProfile::GetMode() const
{
	return Mode;
}

void FHierarchyTableBlendProfile::ConstructBlendProfile(const TObjectPtr<UBlendProfile> OutBlendProfile) const
{
	OutBlendProfile->SetSkeleton(Skeleton);
	OutBlendProfile->Mode = Mode;

	for (int32 BoneIndex = 0; BoneIndex < BoneBlendWeights.Num(); ++BoneIndex)
	{
		const float BlendWeight = BoneBlendWeights[BoneIndex];

		OutBlendProfile->SetBoneBlendScale(BoneIndex, BlendWeight, false /* bRecurse */, true /* bCreate */);
	}
}

bool FHierarchyTableBlendProfile::IsValidBoneIndex(const int32 BoneIndex) const
{
	return BoneBlendWeights.IsValidIndex(BoneIndex);
}
