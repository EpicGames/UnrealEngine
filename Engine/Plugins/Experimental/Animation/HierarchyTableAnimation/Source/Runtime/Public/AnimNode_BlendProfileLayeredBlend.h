// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimNodeBase.h"
#include "HierarchyTableBlendProfile.h"
#include "AnimNode_BlendProfileLayeredBlend.generated.h"

#define UE_API HIERARCHYTABLEANIMATIONRUNTIME_API


class UBlendProfileStandalone;

/** The space in which bone rotations are blended */
UENUM(BlueprintType)
enum class EBlendProfileRotationBlendSpace : uint8
{
	/** Blend bone rotations in local space */
	LocalSpace	UMETA(DisplayName = "Local Space"),
	/** Blend bone rotations in mesh space */
	MeshSpace	UMETA(DisplayName = "Mesh Space"),
	/** Blend bone rotations in root space (mesh space relative to the root bone) */
	RootSpace	UMETA(DisplayName = "Root Space"),
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_BlendProfileLayeredBlend : public FAnimNode_Base
{
	GENERATED_BODY()

public:
	/** The source pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links)
	FPoseLink BasePose;

	/** The target pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Links, meta = (BlueprintCompilerGeneratedDefaults))
	FPoseLink BlendPose;

	/**
	 * The blend profile mask asset to use to control layering of the pose, curves, and attributes
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Config, meta = (PinHiddenByDefault))
	TObjectPtr<UBlendProfileStandalone> BlendProfileAsset;

	/** The space in which bone rotations are blended */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Config, meta = (PinHiddenByDefault))
	EBlendProfileRotationBlendSpace RotationBlendSpace = EBlendProfileRotationBlendSpace::LocalSpace;

#if WITH_EDITORONLY_DATA
	/** Whether to blend bone rotations in mesh space or in local space */
	UE_DEPRECATED(5.8, "bMeshSpaceRotationBlend is deprecated, use RotationBlendSpace instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use RotationBlendSpace instead."))
	bool bMeshSpaceRotationBlend_DEPRECATED = false;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Config)
	bool bCustomCurveBlending;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Config, meta = (EditCondition = "bCustomCurveBlending"))
	TEnumAsByte<enum ECurveBlendOption::Type> CurveBlendingOption;

protected:
	// Guids for skeleton used to determine whether the HierarchyTableBlendProfile need rebuilding
	UPROPERTY()
	FGuid SkeletonGuid;

	// Guid for virtual bones used to determine whether the HierarchyTableBlendProfile need rebuilding
	UPROPERTY()
	FGuid VirtualBoneGuid;

	// Guid for mask table used to determine whether the HierarchyTableBlendProfile need rebuilding
	UPROPERTY()
	FGuid MaskTableGuid;

	// Serial number of the required bones container
	uint16 RequiredBonesSerialNumber = 0;

	/** The weight of target pose */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Runtime, meta = (BlueprintCompilerGeneratedDefaults, PinShownByDefault))
	float BlendWeight;

	/** Whether to incorporate the per-bone blend weight of the root bone when lending root motion */
	UPROPERTY(EditAnywhere, Category = Config)
	bool bBlendRootMotionBasedOnRootBone;

public:
	FAnimNode_BlendProfileLayeredBlend()
		: bCustomCurveBlending(false)
		, CurveBlendingOption(ECurveBlendOption::Override)
		, BlendWeight(1.0f)
		, bBlendRootMotionBasedOnRootBone(true)
	{
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
	// this is a requirement for clang to compile without warnings.
	~FAnimNode_BlendProfileLayeredBlend() = default;
	FAnimNode_BlendProfileLayeredBlend(const FAnimNode_BlendProfileLayeredBlend&) = default;
	FAnimNode_BlendProfileLayeredBlend(FAnimNode_BlendProfileLayeredBlend&&) = default;
	FAnimNode_BlendProfileLayeredBlend& operator=(const FAnimNode_BlendProfileLayeredBlend&) = default;
	FAnimNode_BlendProfileLayeredBlend& operator=(FAnimNode_BlendProfileLayeredBlend&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// FAnimNode_Base interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

private:
	// Check whether per-bone blend weights are valid according to the skeleton (GUID check)
	UE_API bool ArePerBoneBlendWeightsValid(const USkeleton* InSkeleton) const;

	// Update cached data if required
	UE_API void UpdateCachedBoneData(const FBoneContainer& RequiredBones, USkeleton* Skeleton);

	friend class UAnimGraphNode_BlendProfileLayeredBlend;

	UE_API void UpdateDesiredBoneWeight();

	TArray<float> DesiredBoneBlendWeights;
	TArray<float> CurrentBoneBlendWeights;

	UPROPERTY()
	TObjectPtr<UBlendProfileStandalone> CachedBlendProfile;
};

#undef UE_API
