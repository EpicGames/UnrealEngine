// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HierarchyTable.h"
#include "HierarchyTableBlendProfile.h"

#include "BlendProfileStandalone.generated.h"

#define UE_API HIERARCHYTABLEANIMATIONRUNTIME_API

UENUM()
enum class EBlendProfileStandaloneType
{
	WeightFactor,
	TimeFactor,
	BlendMask
};

UCLASS(MinimalAPI, EditInlineNew, BlueprintType)
class UBlendProfileStandalone : public UObject
{
	GENERATED_BODY()

public:
	// Begin UObject
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
#endif
	// End UObject

#if WITH_EDITOR
	UE_API void UpdateHierarchy();

	UE_API void UpdateCachedData();
	
	UE_API bool IsCachedDataStale() const;
#endif

	UE_API TObjectPtr<USkeleton> GetSkeleton() const;

public:
	UPROPERTY(VisibleAnywhere, Category=Default, AssetRegistrySearchable)
	EBlendProfileStandaloneType Type;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<UHierarchyTable> Table;
#endif // WITH_EDITORONLY_DATA

	// The flattened version of the table data for use at runtime instead of,
	// slowly traversing the table tree.
	UPROPERTY()
	FBlendProfileStandaloneCachedData CachedBlendProfileData;

private:
#if WITH_EDITOR
	void SetSkeleton(TObjectPtr<USkeleton> InSkeleton);

	void HandleSkeletonHierarchyChanged();
#endif // WITH_EDITOR

	UPROPERTY(VisibleAnywhere, Category=Default)
	TObjectPtr<USkeleton> Skeleton;

	UPROPERTY()
	FGuid SkeletonHierarchyGuid;

	UPROPERTY()
	FGuid SkeletonVirtualBonesHierarchyGuid;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
    FGuid CachedEntriesGuid;
#endif
};

#undef UE_API
