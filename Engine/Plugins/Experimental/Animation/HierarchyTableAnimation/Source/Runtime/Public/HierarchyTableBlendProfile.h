// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AttributesContainer.h"
#include "Animation/BlendProfile.h"
#include "HierarchyTable.h"

#include "HierarchyTableBlendProfile.generated.h"

#define UE_API HIERARCHYTABLEANIMATIONRUNTIME_API

USTRUCT()
struct FMaskedAttributeWeightSerialised
{
	GENERATED_BODY()

	UPROPERTY()
	FName AttributeNamespace;
	
	UPROPERTY()
	FName AttributeName;
	
	UPROPERTY()
	int32 AttributeIndex = INDEX_NONE;

	UPROPERTY()
	float Weight = 0.0f;
};

USTRUCT()
struct FMaskedCurveWeightSerialised
{
	GENERATED_BODY()

	UPROPERTY()
	FName CurveName;

	UPROPERTY()
	float Weight = 0.0f;
};

USTRUCT()
struct FBlendProfileStandaloneCachedData
{
	GENERATED_BODY()

public:
	UE_API void UnpackCachedData();

	UE_API void Init(TObjectPtr<UHierarchyTable> InHierarchyTable, const EBlendProfileMode InMode);

	UE_API void Reset();

	struct FNamedFloat
	{
		FName Name;
		float Value;

		FNamedFloat(const FName InName, const float InValue)
			: Name(InName)
			, Value(InValue)
		{
		}
	};

	struct FMaskedAttributeWeight
	{
		UE::Anim::FAttributeId Attribute;
		float Weight;

		FMaskedAttributeWeight(const UE::Anim::FAttributeId InAttribute, const float InWeight)
			: Attribute(InAttribute)
			, Weight(InWeight)
		{
		}
	};

	UE_API const TArray<float>& GetBoneBlendWeights() const;

	UE_API const UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement>& GetCurveBlendWeights() const;

	UE_API const TArray<FMaskedAttributeWeight>& GetAttributeBlendWeights() const;

	UE_API void ConstructBlendProfile(const TObjectPtr<UBlendProfile> OutBlendProfile) const;

	UE_API bool IsValidBoneIndex(const int32 BoneIndex) const;
	
	UE_API TObjectPtr<USkeleton> GetSkeleton() const;

private:
	UPROPERTY()
	TObjectPtr<USkeleton> Skeleton;

	UPROPERTY()
	TArray<float> BoneBlendWeights;

	UPROPERTY()
	TArray<FMaskedCurveWeightSerialised> SerializedCurveBlendWeights;

	UPROPERTY()
	TArray<FMaskedAttributeWeightSerialised> SerializedAttributeBlendWeights;

	UPROPERTY()
	EBlendProfileMode Mode = EBlendProfileMode::WeightFactor;

private:
	UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement> CurveBlendWeights;

	TArray<FMaskedAttributeWeight> AttributeBlendWeights;
};

class FHierarchyTableBlendProfile : public IBlendProfileInterface
{
public:
	FHierarchyTableBlendProfile() = default;
	virtual ~FHierarchyTableBlendProfile() = default;

	UE_API FHierarchyTableBlendProfile(TObjectPtr<UHierarchyTable> InHierarchyTable, const EBlendProfileMode InMode);

	UE_API FHierarchyTableBlendProfile(const TObjectPtr<USkeleton> InSkeleton, const EBlendProfileMode InMode);

	// IBlendProfileInterface
	UE_API virtual float GetBoneBlendScale(int32 InBoneIdx) const override;
	UE_API virtual int32 GetNumBlendEntries() const override;
	UE_API virtual int32 GetPerBoneInterpolationIndex(const FCompactPoseBoneIndex& InCompactPoseBoneIndex, const FBoneContainer& BoneContainer, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const override;
	UE_API virtual int32 GetPerBoneInterpolationIndex(const FSkeletonPoseBoneIndex InSkeletonBoneIndex, const USkeleton* TargetSkeleton, const IInterpolationIndexProvider::FPerBoneInterpolationData* Data) const override;
	UE_API virtual EBlendProfileMode GetMode() const override;
	UE_API virtual TObjectPtr<USkeleton> GetSkeleton() const override;

	UE_API const UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement>& GetCurveBlendWeights() const;
	UE_API const TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight>& GetAttributeBlendWeights() const;

	UE_API void ConstructBlendProfile(const TObjectPtr<UBlendProfile> OutBlendProfile) const;

	UE_API bool IsValidBoneIndex(const int32 BoneIndex) const;

private:
	TObjectPtr<USkeleton> Skeleton;

	TArray<float> BoneBlendWeights;

	UE::Anim::TNamedValueArray<FDefaultAllocator, UE::Anim::FCurveElement> CurveBlendWeights;

	TArray<FBlendProfileStandaloneCachedData::FMaskedAttributeWeight> AttributeBlendWeights;

	EBlendProfileMode Mode = EBlendProfileMode::WeightFactor;
};

#undef UE_API
