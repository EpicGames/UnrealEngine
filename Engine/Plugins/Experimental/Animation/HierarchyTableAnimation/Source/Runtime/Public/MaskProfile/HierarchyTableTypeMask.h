// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyTableType.h"

#include "HierarchyTableTypeMask.generated.h"

USTRUCT(DisplayName = "Mask")
struct FHierarchyTable_ElementType_Mask final : public FHierarchyTable_ElementType
{
	GENERATED_BODY()

	FHierarchyTable_ElementType_Mask()
		: Value(1.0f)
	{
	}

	UPROPERTY()
	float Value;
};