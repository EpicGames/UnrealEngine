// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyTableType.h"

#include "SkeletonHierarchyTableType.generated.h"

class USkeleton;

USTRUCT(DisplayName = "Skeleton")
struct FHierarchyTable_TableType_Skeleton final : public FHierarchyTable_TableType
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Default)
	TObjectPtr<USkeleton> Skeleton;
};

UENUM()
enum class ESkeletonHierarchyTable_TablePayloadEntryType
{
	Bone,
	Curve,
	Attribute
};

USTRUCT()
struct FHierarchyTable_TablePayloadType_Skeleton final : public FHierarchyTable_TablePayloadType
{
	GENERATED_BODY()

	UPROPERTY()
	ESkeletonHierarchyTable_TablePayloadEntryType EntryType = ESkeletonHierarchyTable_TablePayloadEntryType::Bone;
};