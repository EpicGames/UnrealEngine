// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

public class HierarchyTableAnimationUncookedOnly : ModuleRules
{
	public HierarchyTableAnimationUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "HTAnimUncook";

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
				"AnimGraph",
				"AnimGraphRuntime",
				"AnimationBlueprintLibrary",
				"ToolMenus"
			});

		PrivateDependencyModuleNames.AddAll(
			"Core",
			"CoreUObject",
			"Engine",
			"InputCore",
			"Slate",
			"SlateCore",
			"UnrealEd",
			"HierarchyTableRuntime",
			"HierarchyTableEditor",
			"HierarchyTableAnimationRuntime"
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
						"BlueprintGraph",
						"EditorFramework",
						"Kismet",
						"UnrealEd",
				}
			);
		}
	}
}
