// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_BlendProfileLayeredBlend.h"
#include "ToolMenus.h"
#include "Kismet2/BlueprintEditorUtils.h"

#include "AnimGraphCommands.h"
#include "ScopedTransaction.h"

#include "DetailLayoutBuilder.h"
#include "Kismet2/CompilerResultsLog.h"

#include "HierarchyTable.h"
#include "MaskProfile/HierarchyTableTypeMask.h"
#include "BlendProfileStandalone.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_BlendProfileLayeredBlend)

#define LOCTEXT_NAMESPACE "AnimGraphNode_BlendProfileLayeredBlend"

UAnimGraphNode_BlendProfileLayeredBlend::UAnimGraphNode_BlendProfileLayeredBlend(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAnimGraphNode_BlendProfileLayeredBlend::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::BlendProfileLayeredBlendRotationSpace)
		{
			// Migrate deprecated bMeshSpaceRotationBlend to the new RotationBlendSpace enum
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Node.bMeshSpaceRotationBlend_DEPRECATED)
			{
				Node.RotationBlendSpace = EBlendProfileRotationBlendSpace::MeshSpace;
				Node.bMeshSpaceRotationBlend_DEPRECATED = false;
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

FLinearColor UAnimGraphNode_BlendProfileLayeredBlend::GetNodeTitleColor() const
{
	return FLinearColor(0.75f, 0.75f, 0.75f);
}

FText UAnimGraphNode_BlendProfileLayeredBlend::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_BlendProfileLayeredBlend_Tooltip", "Profile Blend");
}

FText UAnimGraphNode_BlendProfileLayeredBlend::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_BlendProfileLayeredBlend_Title", "Profile Blend");
}

FString UAnimGraphNode_BlendProfileLayeredBlend::GetNodeCategory() const
{
	return TEXT("Animation|Blends");
}

void UAnimGraphNode_BlendProfileLayeredBlend::PreloadRequiredAssets()
{
	if (Node.BlendProfileAsset)
	{
		PreloadObject(Node.BlendProfileAsset);
	}

	Super::PreloadRequiredAssets();
}

void UAnimGraphNode_BlendProfileLayeredBlend::ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog)
{
	UAnimGraphNode_Base::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	bool bCompilationError = false;

	const TObjectPtr<UBlendProfileStandalone> BlendProfileAsset = Node.BlendProfileAsset;

	if (BlendProfileAsset)
	{
		if (BlendProfileAsset->Type != EBlendProfileStandaloneType::BlendMask)
		{
			MessageLog.Error(*LOCTEXT("InvalidBlendProfileAssetType", "@@ uses a blend profile asset of the incorrect type, expected Blend Mask type.").ToString(), this);
			bCompilationError = true;
		}

		if (BlendProfileAsset->GetSkeleton() != ForSkeleton)
		{
			MessageLog.Error(*LOCTEXT("InvalidBlendProfileSkeleton", "@@ uses a blend profile asset with the wrong skeleton.").ToString(), this);
			bCompilationError = true;
		}
	}
}

#undef LOCTEXT_NAMESPACE
