// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyTableAnimationUncookedOnlyModule.h"
#include "Modules/ModuleManager.h"

void FHierarchyTableAnimationUncookedOnlyModule::StartupModule()
{
}

void FHierarchyTableAnimationUncookedOnlyModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FHierarchyTableAnimationUncookedOnlyModule, HierarchyTableAnimationUncookedOnly)
