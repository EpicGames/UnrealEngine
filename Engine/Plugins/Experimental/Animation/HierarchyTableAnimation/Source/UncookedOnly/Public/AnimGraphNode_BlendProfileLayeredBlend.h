// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_BlendListBase.h"
#include "AnimNode_BlendProfileLayeredBlend.h"
#include "AnimGraphNode_BlendProfileLayeredBlend.generated.h"

UCLASS(MinimalAPI)
class UAnimGraphNode_BlendProfileLayeredBlend : public UAnimGraphNode_BlendListBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_BlendProfileLayeredBlend Node;

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject interface

	// UEdGraphNode interface
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual FString GetNodeCategory() const override;
	virtual void PreloadRequiredAssets() override;
	virtual void ValidateAnimNodeDuringCompilation(class USkeleton* ForSkeleton, class FCompilerResultsLog& MessageLog) override;
	// End of UAnimGraphNode_Base interface
};
